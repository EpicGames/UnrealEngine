// Copyright Epic Games, Inc. All Rights Reserved.


#include "IConcertClientTransactionBridge.h"
#include "IConcertSyncClientModule.h"

#include "UObject/Package.h"
#include "USDLog.h"
#include "USDStageActor.h"
#include "USDTransactor.h"

namespace UE
{
	namespace UsdMultiUser
	{
		namespace Private
		{
			ETransactionFilterResult TransactionFilterFunction( const FConcertTransactionFilterArgs& FilterArgs )
			{
				const UObject* ObjectToFilter = FilterArgs.ObjectToFilter;
				if ( const UUsdTransactor* Transactor = Cast<UUsdTransactor>( ObjectToFilter ); Transactor && Transactor->IsInA( AUsdStageActor::StaticClass() ) )
				{
					return ETransactionFilterResult::IncludeObject;
				}

				// Allow transient objects only if they have the enable tag
				// Having this tag means they are attached to and owned by an AUsdStageActor's hierarchy
				if ( ObjectToFilter && ObjectToFilter->HasAnyFlags( RF_Transient ) )
				{
					if ( const AActor* Actor = Cast<AActor>( ObjectToFilter ) )
					{
						if ( Actor->Tags.Contains( UE::UsdTransactor::ConcertSyncEnableTag ) )
						{
							return ETransactionFilterResult::IncludeObject;
						}
					}
					else if ( const UActorComponent* Component = Cast<UActorComponent>( ObjectToFilter ) )
					{
						if ( Component->ComponentTags.Contains( UE::UsdTransactor::ConcertSyncEnableTag ) )
						{
							return ETransactionFilterResult::IncludeObject;
						}
					}
				}

				return ETransactionFilterResult::UseDefault;
			}

			void EnableTransactorFilters()
			{
				IConcertClientTransactionBridge& TransactionBridge = IConcertSyncClientModule::Get().GetTransactionBridge();
				TransactionBridge.RegisterTransactionFilter( TEXT( "USD-Main-Filters" ), FOnFilterTransactionDelegate::CreateStatic( &TransactionFilterFunction ) );

				UE_LOGF( LogUsd, Log, "Added ConcertSync filters for UUsdTransactor" );
			}

			void DisableTransactorFilters()
			{
				IConcertClientTransactionBridge& TransactionBridge = IConcertSyncClientModule::Get().GetTransactionBridge();
				TransactionBridge.UnregisterTransactionFilter( TEXT( "USD-Main-Filters" ) );

				UE_LOGF( LogUsd, Log, "Removed ConcertSync filters for UUsdTransactor" );
			}
		}
	}
}

/**
 * Module that adds multi user synchronization to the USD Multi User plugin.
 */
class FUsdMultiUserModule : public IModuleInterface
{
private:
	virtual void StartupModule() override
	{
		UE::UsdMultiUser::Private::EnableTransactorFilters();
	}

	virtual void ShutdownModule() override
	{
		// If we're shutting down the classes will already be gone anyway
		if ( !IsEngineExitRequested() )
		{
			UE::UsdMultiUser::Private::DisableTransactorFilters();
		}
	}
};


IMPLEMENT_MODULE(FUsdMultiUserModule, USDMultiUser);
