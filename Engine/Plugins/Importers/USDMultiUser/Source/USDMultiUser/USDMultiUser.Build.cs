// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class USDMultiUser : ModuleRules
{
	public USDMultiUser(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ConcertSyncClient",
				"ConcertSyncCore",
				"Engine",
				"UnrealUSDWrapper",
				"USDClasses",
				"USDStage",
				"USDUtilities",
			}
		);
	}
}
