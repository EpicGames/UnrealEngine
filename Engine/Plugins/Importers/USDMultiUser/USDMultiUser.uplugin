{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "USD Multi-User synchronization",
	"Description": "Enables opt-in multi-user synchronization for the USD Importer plugin.",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
    "EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
    "SupportedTargetPlatforms": [
		"Win64", "Mac", "Linux"
	],
    "Modules" :
	[
		{
			"Name": "USDMultiUser",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins":
	[
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "USDImporter",
			"Enabled": true
		}
	]
}